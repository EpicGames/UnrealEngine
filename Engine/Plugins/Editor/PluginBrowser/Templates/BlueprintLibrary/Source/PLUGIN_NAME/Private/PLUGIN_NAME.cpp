// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAME.h"

#define LOCTEXT_NAMESPACE "FPLUGIN_NAMEModule"

void FPLUGIN_NAMEModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	
}

void FPLUGIN_NAMEModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FPLUGIN_NAMEModule, PLUGIN_NAME)