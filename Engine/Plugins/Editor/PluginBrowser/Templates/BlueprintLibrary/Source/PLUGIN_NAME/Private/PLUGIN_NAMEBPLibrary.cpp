// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMEBPLibrary.h"
#include "PLUGIN_NAME.h"

UPLUGIN_NAMEBPLibrary::UPLUGIN_NAMEBPLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

float UPLUGIN_NAMEBPLibrary::PLUGIN_NAMESampleFunction(float Param)
{
	return -1;
}

