// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "PLUGIN_NAMEBPLibrary.generated.h"

/* 
*	Function library class.
*	Each function in it is expected to be static and represents blueprint node that can be called in any blueprint.
*
*	When declaring function you can define metadata for the node. Key function specifiers will be BlueprintPure and BlueprintCallable.
*	BlueprintPure - means the function does not affect the owning object in any way and thus creates a node without Exec pins.
*	BlueprintCallable - makes a function which can be executed in Blueprints - Thus it has Exec pins.
*	DisplayName - full name of the node, shown when you mouse over the node and in the blueprint drop down menu.
*				Its lets you name the node using characters not allowed in C++ function names.
*	CompactNodeTitle - the word(s) that appear on the node.
*	Keywords -	the list of keywords that helps you to find node when you search for it using Blueprint drop-down menu. 
*				Good example is "Print String" node which you can find also by using keyword "log".
*	Category -	the category your node will be under in the Blueprint drop-down menu.
*
*	For more info on custom blueprint nodes visit documentation:
*	https://wiki.unrealengine.com/Custom_Blueprint_Node_Creation
*/
UCLASS()
class UPLUGIN_NAMEBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Execute Sample function", Keywords = "PLUGIN_NAME sample test testing"), Category = "PLUGIN_NAMETesting")
	static float PLUGIN_NAMESampleFunction(float Param);
};
